// Copyright Epic Games, Inc. All Rights Reserved.

#include "SZenStatusBar.h"

#include "Async/Future.h"
#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "DerivedDataCacheInterface.h"
#include "ZenEditor.h"
#include "DerivedDataInformation.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxDefs.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Framework/SlateDelegates.h"
#include "HAL/PlatformCrt.h"
#include "ISettingsModule.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Math/Color.h"
#include "Math/UnitConversion.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/Attribute.h"
#include "Misc/CoreMisc.h"
#include "Modules/ModuleManager.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateTypes.h"
#include "Textures/SlateIcon.h"
#include "ToolMenu.h"
#include "ToolMenuContext.h"
#include "ToolMenuSection.h"
#include "ToolMenus.h"
#include "Types/WidgetActiveTimerDelegate.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"
#include "ZenDashboardLauncher.h"
#include "ZenServiceInstanceManager.h"
#include "Experimental/ZenServerInterface.h"

class SWidget;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "ZenStatusBar"


TSharedRef<FUICommandList> FZenStausBarCommands::ActionList(new FUICommandList());
TSharedPtr<UE::Zen::FServiceInstanceManager> FZenStausBarCommands::ServiceInstanceManager;

static const FName ZenMenuName(TEXT("ZenStatusBarMenu"));
static const FName ZenMenuServerSectionName(TEXT("ZenStatusBarMenu.ServerSection"));
static const FName ZenMenuStoreSectionName(TEXT("ZenStatusBarMenu.StoreSection"));
static const FName ZenMenuToolsSectionName(TEXT("ZenStatusBarMenu.ToolsSection"));
static const FName ZenMenuCacheSectionName(TEXT("ZenStatusBarMenu.CacheSection"));
static const FName ZenMenuServerActionsSubMenuName(TEXT("ZenStatusBarMenu.ServerActions"));


FZenStausBarCommands::FZenStausBarCommands()
	: TCommands<FZenStausBarCommands>
	(
		"ZenSettings",
		NSLOCTEXT("Contexts", "Zen", "Zen"),
		"LevelEditor",
		FAppStyle::GetAppStyleSetName()
	)
{}

void FZenStausBarCommands::RegisterCommands()
{
	ServiceInstanceManager = MakeShared<UE::Zen::FServiceInstanceManager>();

	UI_COMMAND(ChangeCacheSettings, "Cache Settings", "Opens a dialog to change Cache settings.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ViewCacheStatistics, "Cache Statistics", "Opens the Cache Statistics panel.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ViewResourceUsage, "Cache Resource Usage", "Opens the Cache Resource Usage panel.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ViewServerStatus, "Server Status", "Opens the Zen Server Status panel.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ViewZenStore, "View Store", "Opens the Zen Store page in your browser.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(LaunchZenDashboard, "Launch Dashboard", "Launches the Zen Dashboard utility.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(StartZenServer, "Start Server", "Starts the Zen Server.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(StopZenServer, "Stop Server", "Stops the Zen Server.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(RestartZenServer, "Restart Server", "Restarts the Zen Server.", EUserInterfaceActionType::Button, FInputChord());

	ActionList->MapAction(
		ChangeCacheSettings,
		FExecuteAction::CreateStatic(&FZenStausBarCommands::ChangeCacheSettings_Clicked)
	);

	ActionList->MapAction(
		ViewCacheStatistics,
		FExecuteAction::CreateStatic(&FZenStausBarCommands::ViewCacheStatistics_Clicked)
	);

	ActionList->MapAction(
		ViewResourceUsage,
		FExecuteAction::CreateStatic(&FZenStausBarCommands::ViewResourceUsage_Clicked)
	);

	ActionList->MapAction(
		ViewServerStatus,
		FExecuteAction::CreateStatic(&FZenStausBarCommands::ViewServerStatus_Clicked)
	);

	ActionList->MapAction(
		ViewZenStore,
		FExecuteAction::CreateStatic(&FZenStausBarCommands::ViewZenStore_Clicked)
	);

	ActionList->MapAction(
		LaunchZenDashboard,
		FExecuteAction::CreateStatic(&FZenStausBarCommands::LaunchZenDashboard_Clicked)
	);

	ActionList->MapAction(
		StartZenServer,
		FExecuteAction::CreateStatic(&FZenStausBarCommands::StartZenServer_Clicked)
	);

	ActionList->MapAction(
		StopZenServer,
		FExecuteAction::CreateStatic(&FZenStausBarCommands::StopZenServer_Clicked)
	);

	ActionList->MapAction(
		RestartZenServer,
		FExecuteAction::CreateStatic(&FZenStausBarCommands::RestartZenServer_Clicked)
	);
}

void FZenStausBarCommands::ChangeCacheSettings_Clicked()
{
	FModuleManager::LoadModuleChecked<ISettingsModule>("Settings").ShowViewer("Editor", "General", "Global");
}

void FZenStausBarCommands::ViewCacheStatistics_Clicked()
{
	FZenEditor& ZenEditor = FModuleManager::LoadModuleChecked<FZenEditor>("ZenEditor");
	ZenEditor.ShowCacheStatisticsTab();
}

void FZenStausBarCommands::ViewResourceUsage_Clicked()
{
	FZenEditor& ZenEditor = FModuleManager::LoadModuleChecked<FZenEditor>("ZenEditor");
	ZenEditor.ShowResourceUsageTab();
}

void FZenStausBarCommands::ViewServerStatus_Clicked()
{
	FZenEditor& ZenEditor = FModuleManager::LoadModuleChecked<FZenEditor>("ZenEditor");
	ZenEditor.ShowZenServerStatusTab();
}

void FZenStausBarCommands::StartZenServer_Clicked()
{
	FZenEditor& ZenEditor = FModuleManager::LoadModuleChecked<FZenEditor>("ZenEditor");
	ZenEditor.StartZenServer();
}

void FZenStausBarCommands::StopZenServer_Clicked()
{
	FZenEditor& ZenEditor = FModuleManager::LoadModuleChecked<FZenEditor>("ZenEditor");
	ZenEditor.StopZenServer();
}

void FZenStausBarCommands::RestartZenServer_Clicked()
{
	FZenEditor& ZenEditor = FModuleManager::LoadModuleChecked<FZenEditor>("ZenEditor");
	ZenEditor.RestartZenServer();
}

void FZenStausBarCommands::ViewZenStore_Clicked()
{
	using namespace UE::Zen;
	TSharedPtr<FZenServiceInstance> ServerInstance = ServiceInstanceManager.Get()->GetZenServiceInstance();

	if (ServerInstance.IsValid())
	{
		FString URL = ServerInstance->GetEndpoint().GetURL();
		FPlatformProcess::LaunchURL(*FString::Printf(TEXT("%s/dashboard/?"), *URL), nullptr, nullptr);
	}
}

void FZenStausBarCommands::LaunchZenDashboard_Clicked()
{
	using namespace UE::Zen;
	FZenDashboardLauncher::Get()->StartZenDashboard(FZenDashboardLauncher::Get()->GetZenDashboardApplicationPath());
}


TSharedRef<SWidget> SZenStatusBarWidget::CreateStatusBarMenu()
{
	UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(ZenMenuName, NAME_None, EMultiBoxType::Menu, false);

	{
		FToolMenuSection& Section = Menu->AddSection(ZenMenuToolsSectionName, LOCTEXT("ZenMenuToolsSection", "Tools"));

		Section.AddMenuEntry(
			FZenStausBarCommands::Get().LaunchZenDashboard,
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Zen.Dashboard.Icon")
		);
	}

	{
		FToolMenuSection& Section = Menu->AddSection(ZenMenuServerSectionName, LOCTEXT("ZenMenuServerSection", "Server"));

		Section.AddMenuEntry(
			FZenStausBarCommands::Get().ViewServerStatus,
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Zen.Server")
		);

		{
			Section.AddSubMenu(TEXT("ZenServerActionSubMenu"), LOCTEXT("ZenServerActions", "Server Actions"), LOCTEXT("ZenServerActionsSubMenu", "Server Actions"),
				FNewToolMenuDelegate::CreateLambda([this](UToolMenu* InSubMenu)
					{
						using namespace UE::Zen;
						TSharedPtr<FZenServiceInstance> ServerInstance = ServiceInstanceManager.Get()->GetZenServiceInstance();
						FToolMenuSection& Section = InSubMenu->AddSection(NAME_None);
						
						if (ServerInstance.IsValid() && ServerInstance->IsServiceRunning())
						{
							Section.AddMenuEntry(
								FZenStausBarCommands::Get().StopZenServer,
								TAttribute<FText>(),
								TAttribute<FText>(),
								FSlateIcon(FAppStyle::GetAppStyleSetName(), "Zen.Server.Stop")
							);

							Section.AddMenuEntry(
								FZenStausBarCommands::Get().RestartZenServer,
								TAttribute<FText>(),
								TAttribute<FText>(),
								FSlateIcon(FAppStyle::GetAppStyleSetName(), "Zen.Server.Restart")
							);
						}
						else
						{
							Section.AddMenuEntry(
								FZenStausBarCommands::Get().StartZenServer,
								TAttribute<FText>(),
								TAttribute<FText>(),
								FSlateIcon(FAppStyle::GetAppStyleSetName(), "Zen.Server.Start")
							);
						}
					}));
		}
	}

	{
		FToolMenuSection& Section = Menu->AddSection(ZenMenuStoreSectionName, LOCTEXT("ZenMenuStoreSection", "Store"));

		Section.AddMenuEntry(
			FZenStausBarCommands::Get().ViewZenStore,
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Zen.Store")
		);
	}

	{
		FToolMenuSection& Section = Menu->AddSection(ZenMenuCacheSectionName, LOCTEXT("ZenMenuDerivedDataSection", "Cache"));

		Section.AddMenuEntry(
			FZenStausBarCommands::Get().ViewCacheStatistics,
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "DerivedData.Cache.Statistics")
		);
	
		Section.AddMenuEntry(
			FZenStausBarCommands::Get().ViewResourceUsage,
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "DerivedData.ResourceUsage")
		);

		Section.AddMenuEntry(
			FZenStausBarCommands::Get().ChangeCacheSettings,
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "DerivedData.Cache.Settings")
		);
	}

	return UToolMenus::Get()->GenerateWidget(ZenMenuName, FToolMenuContext(FZenStausBarCommands::ActionList));
}

void SZenStatusBarWidget::Construct(const FArguments& InArgs)
{	
	ServiceInstanceManager = MakeShared<UE::Zen::FServiceInstanceManager>();

	this->ChildSlot
	[
		SNew(SComboButton)
		.ContentPadding(FMargin(6.0f, 0.0f))
		.MenuPlacement(MenuPlacement_AboveAnchor)
		.ComboButtonStyle(&FAppStyle::Get().GetWidgetStyle<FComboButtonStyle>("SimpleComboButton"))
		.ButtonContent()
		[

			SNew(SHorizontalBox)
	
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(0, 0, 3, 0)
			[
				SNew(SOverlay)
				+ SOverlay::Slot()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Top)
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image_Lambda([this] { return GetServerStateBackgroundIcon();  })
					.ToolTipText_Lambda([this] { return GetServerStateToolTipText(); })
				]

				+ SOverlay::Slot()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Top)
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image_Lambda([this] { return GetServerStateBadgeIcon();  })
					.ToolTipText_Lambda([this] { return GetServerStateToolTipText(); })
				]

				+ SOverlay::Slot()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Top)
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Zen.Status.Uploading"))
					.ColorAndOpacity_Lambda([this] { return IsUploading? FLinearColor::White.CopyWithNewOpacity(FMath::MakePulsatingValue(ElapsedUploadTime, 2)) : FLinearColor(0,0,0,0); })
					.ToolTipText_Lambda([this] { return GetServerStateToolTipText(); })
				]

				+ SOverlay::Slot()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Top)
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Zen.Status.Downloading"))
					.ColorAndOpacity_Lambda([this] { return IsDownloading ? FLinearColor::White.CopyWithNewOpacity(FMath::MakePulsatingValue(ElapsedDownloadTime, 2)) : FLinearColor(0, 0, 0, 0); })
					.ToolTipText_Lambda([this] { return GetServerStateToolTipText(); })
				]				
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(0, 0, 10, 0)
			[
				SNew(STextBlock)
				.Text_Lambda([this] { return GetTitleText(); })
				.ToolTipText_Lambda([this] { return GetTitleToolTipText(); })
			]
		]
		.OnGetMenuContent(FOnGetContent::CreateRaw(this, &SZenStatusBarWidget::CreateStatusBarMenu))
	];

	RegisterActiveTimer(0.2f, FWidgetActiveTimerDelegate::CreateSP(this, &SZenStatusBarWidget::UpdateBusyIndicator));
	RegisterActiveTimer(5.0f, FWidgetActiveTimerDelegate::CreateSP(this, &SZenStatusBarWidget::UpdateWarnings));
}

EActiveTimerReturnType SZenStatusBarWidget::UpdateBusyIndicator(double InCurrentTime, float InDeltaTime)
{	
	using namespace UE::Zen;
	TSharedPtr<FZenServiceInstance> ServerInstance = ServiceInstanceManager.Get()->GetZenServiceInstance();

	static int64 LastTotalHits=0;
	static int64 LastTotalReads=0;
	static int64 LastTotalWrites=0;

	if (ServerInstance.IsValid() && ServerInstance->IsServiceRunning())
	{
		FZenCacheStats CacheStats;
		ServerInstance->GetCacheStats(CacheStats);

		FZenProjectStats ProjectStats;
		ServerInstance->GetProjectStats(ProjectStats);

		const int64 TotalCacheHits = CacheStats.General.Hits + CacheStats.General.CidHits;
		const int64 TotalCacheWrites = CacheStats.General.Writes + CacheStats.General.CidWrites;
		const int64 TotalProjectReads = ProjectStats.General.Project.ReadCount + ProjectStats.General.Oplog.ReadCount;
		const int64 TotalProjectWrites = ProjectStats.General.Project.ReadCount + ProjectStats.General.Oplog.ReadCount;

		const int64 TotalHits = TotalCacheHits;
		const int64 TotalReads = TotalProjectReads;
		const int64 TotalWrites = TotalCacheWrites + TotalProjectWrites;
			
		IsRunning = true;
		IsDownloading = CacheStats.Upstream.Reading;
		IsUploading = CacheStats.Upstream.Writing;
		IsReading = TotalReads > LastTotalReads;
		IsWriting = TotalWrites > LastTotalWrites;

		LastTotalHits = TotalHits;
		LastTotalReads = TotalReads;
		LastTotalWrites = TotalWrites;
	}
	else
	{
		IsRunning = false;
		IsDownloading = false;
		IsUploading = false;
		IsReading = false;
		IsWriting = false;

		LastTotalHits = 0;
		LastTotalReads = 0;
		LastTotalWrites = 0;
	}

	IsBusy = IsUploading || IsDownloading || IsReading || IsWriting;

	FDerivedDataInformation::UpdateRemoteCacheState();

	IsBusy			= GetDerivedDataCache()->AnyAsyncRequestsRemaining();
	IsUploading		= FDerivedDataInformation::IsUploading() && FDerivedDataInformation::GetRemoteCacheState() == ERemoteCacheState::Busy;
	IsDownloading	= FDerivedDataInformation::IsDownloading() && FDerivedDataInformation::GetRemoteCacheState() == ERemoteCacheState::Busy;

	if (IsUploading)
	{
		ElapsedUploadTime += fmod(InDeltaTime,3600.0);
	}
	else
	{
		ElapsedUploadTime = 0.0;
	}

	if (IsDownloading)
	{
		ElapsedDownloadTime += fmod(InDeltaTime, 3600.0);
	}
	else
	{
		ElapsedDownloadTime = 0.0;
	}

	if (IsBusy)
	{
		ElapsedBusyTime += fmod(InDeltaTime, 3600.0);
	}
	else
	{
		ElapsedBusyTime = 0;
	}

	return EActiveTimerReturnType::Continue;
}

EActiveTimerReturnType SZenStatusBarWidget::UpdateWarnings(double InCurrentTime, float InDeltaTime)
{
	if ( FDerivedDataInformation::GetRemoteCacheState()== ERemoteCacheState::Warning )
	{
		if ( NotificationItem.IsValid() == false || NotificationItem->GetCompletionState()== SNotificationItem::CS_None)
		{
			// No existing notification or the existing one has finished
			TPromise<TWeakPtr<SNotificationItem>> NotificationPromise;

			FNotificationInfo Info(FDerivedDataInformation::GetRemoteCacheWarningMessage());
			Info.bUseSuccessFailIcons = true;
			Info.bFireAndForget = false;
			Info.bUseThrobber = false;
			Info.FadeOutDuration = 0.0f;
			Info.ExpireDuration = 0.0f;

			Info.ButtonDetails.Add(FNotificationButtonInfo(LOCTEXT("OpenSettings", "Open Settings"), FText(), FSimpleDelegate::CreateLambda([NotificationFuture = NotificationPromise.GetFuture().Share()]() {
				FModuleManager::LoadModuleChecked<ISettingsModule>("Settings").ShowViewer("Editor", "General", "Global");

				TWeakPtr<SNotificationItem> NotificationPtr = NotificationFuture.Get();
				if (TSharedPtr<SNotificationItem> Notification = NotificationPtr.Pin())
				{
					Notification->SetCompletionState(SNotificationItem::CS_None);
					Notification->ExpireAndFadeout();
				}
			}),
				SNotificationItem::ECompletionState::CS_Fail));

			NotificationItem = FSlateNotificationManager::Get().AddNotification(Info);

			if (NotificationItem.IsValid())
			{
				NotificationPromise.SetValue(NotificationItem);
				NotificationItem->SetCompletionState(SNotificationItem::CS_Fail);
			}
		}
	}
	else
	{
		// No longer any warnings
		if (NotificationItem.IsValid())
		{
			NotificationItem->SetCompletionState(SNotificationItem::CS_None);
			NotificationItem->ExpireAndFadeout();
		}
	}

	return EActiveTimerReturnType::Continue;
}

FText SZenStatusBarWidget::GetTitleToolTipText() const
{
	return GetServerStateToolTipText();
}

FText SZenStatusBarWidget::GetTitleText() const
{
	return LOCTEXT("ZenStatusBarName", "Zen Server");
}

FText SZenStatusBarWidget::GetServerStateToolTipText() const
{
	FTextBuilder DescBuilder;
	
	if (IsRunning)
	{
		DescBuilder.AppendLineFormat(LOCTEXT("ZenServerStaus", "{0}"), FText::FromString(IsBusy ? TEXT("Busy") : TEXT("Idle")));
	}
	else
	{
		DescBuilder.AppendLineFormat(LOCTEXT("ZenServerStaus", "{0}"), FText::FromString(TEXT("Stopped")));
	}

	return DescBuilder.ToText();
}

const FSlateBrush* SZenStatusBarWidget::GetServerStateBackgroundIcon() const
{
	if (IsRunning)
	{
		return IsBusy ? FAppStyle::Get().GetBrush("Zen.Status.BusyBG") : FAppStyle::Get().GetBrush("Zen.Status.IdleBG");
	}

	return FAppStyle::Get().GetBrush("Zen.Status.UnavailableBG");
}

const FSlateBrush* SZenStatusBarWidget::GetServerStateBadgeIcon() const
{
	if (IsRunning)
	{
		return IsBusy ? FAppStyle::Get().GetBrush("Zen.Status.Busy") : FAppStyle::Get().GetBrush("Zen.Status.Idle");
	}

	return FAppStyle::Get().GetBrush("Zen.Status.Unavailable");
}


#undef LOCTEXT_NAMESPACE