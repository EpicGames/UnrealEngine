// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Styling/SlateColor.h"

class FUICommandInfo;
class FUICommandList;
class SNotificationItem;
class SWidget;
struct FSlateBrush;

namespace UE::Zen { class FServiceInstanceManager; }

class FZenStausBarCommands : public TCommands<FZenStausBarCommands>
{
public:

	FZenStausBarCommands();

	virtual void RegisterCommands() override;

private:

	static void ChangeCacheSettings_Clicked();
	static void ViewCacheStatistics_Clicked();
	static void ViewResourceUsage_Clicked();
	static void ViewServerStatus_Clicked();
	static void ViewZenStore_Clicked();
	static void LaunchZenDashboard_Clicked();
	static void StartZenServer_Clicked();
	static void StopZenServer_Clicked();
	static void RestartZenServer_Clicked();

public:

	TSharedPtr< FUICommandInfo > ChangeCacheSettings;
	TSharedPtr< FUICommandInfo > ViewResourceUsage;
	TSharedPtr< FUICommandInfo > ViewCacheStatistics;
	TSharedPtr< FUICommandInfo > ViewServerStatus;
	TSharedPtr< FUICommandInfo > ViewZenStore;
	TSharedPtr< FUICommandInfo > LaunchZenDashboard;
	TSharedPtr< FUICommandInfo > StartZenServer;
	TSharedPtr< FUICommandInfo > StopZenServer;
	TSharedPtr< FUICommandInfo > RestartZenServer;

	static TSharedRef<FUICommandList> ActionList;
	static TSharedPtr<UE::Zen::FServiceInstanceManager> ServiceInstanceManager;
};


class SZenStatusBarWidget : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SZenStatusBarWidget) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:

	FText						GetTitleToolTipText() const;
	FText						GetTitleText() const;

	FText						GetServerStateToolTipText() const;
	const FSlateBrush*			GetServerStateBadgeIcon() const;
	const FSlateBrush*			GetServerStateBackgroundIcon() const;

	TSharedRef<SWidget>			CreateStatusBarMenu();
	EActiveTimerReturnType		UpdateBusyIndicator(double InCurrentTime, float InDeltaTime);
	EActiveTimerReturnType		UpdateWarnings(double InCurrentTime, float InDeltaTime);

	double ElapsedDownloadTime = 0;
	double ElapsedUploadTime = 0;
	double ElapsedBusyTime = 0;

	bool IsRunning = false;
	bool IsDownloading = false;
	bool IsUploading = false;
	bool IsReading = false;
	bool IsWriting = false;
	bool IsBusy = false;

	TSharedPtr<SNotificationItem> NotificationItem;
	TSharedPtr<UE::Zen::FServiceInstanceManager> ServiceInstanceManager;
};
