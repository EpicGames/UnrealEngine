// Copyright Epic Games, Inc. All Rights Reserved.

#include "ZenDashboardLauncher.h"

#include "IUATHelperModule.h"
#include "Logging/LogMacros.h"
#include "Logging/MessageLog.h"
#include "Misc/Paths.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "FZenDashboardLauncher"

namespace UE::Zen
{

DEFINE_LOG_CATEGORY_STATIC(LogZenDashboardLauncher, Log, All);

TSharedPtr<FZenDashboardLauncher> FZenDashboardLauncher::Instance = nullptr;

FZenDashboardLauncher::FZenDashboardLauncher()
{

}

FZenDashboardLauncher::~FZenDashboardLauncher()
{

}
	
FString FZenDashboardLauncher::GetZenDashboardApplicationPath()
{
	FString Path = FPlatformProcess::GenerateApplicationPath(TEXT("ZenDashboard"), EBuildConfiguration::Development);
	return FPaths::ConvertRelativePathToFull(Path);
}

void FZenDashboardLauncher::StartZenDashboard(const FString& Path, const FString& Parameters)
{
	auto Callback = [](const EStartZenDashboardResult Result) {};
	StartZenDashboard(Path, Parameters, Callback);
}

void FZenDashboardLauncher::StartZenDashboard(const FString& Path, const FString& Parameters, StartZenDashboardCallback Callback)
{
    if (!FPaths::FileExists(Path))
    {
    	BuildZenDashboard(Path, Parameters, Callback);
    	return;
    }
	
	constexpr bool bLaunchDetached = true;
	constexpr bool bLaunchHidden = false;
	constexpr bool bLaunchReallyHidden = false;

	uint32 ProcessID = 0;
	const int32 PriorityModifier = 0;

	const TCHAR* OptionalWorkingDirectory = nullptr;

	void* PipeWriteChild = nullptr;
	void* PipeReadChild = nullptr;
	ZenDashboardHandle = FPlatformProcess::CreateProc(*Path, *Parameters, bLaunchDetached, bLaunchHidden, bLaunchReallyHidden, &ProcessID, PriorityModifier, OptionalWorkingDirectory, PipeWriteChild, PipeReadChild);

	if (ZenDashboardHandle.IsValid())
	{
		UE_LOGF(LogZenDashboardLauncher, Log, "Launched Zen Dashboard executable: %ls %ls", *Path, *Parameters);
		Callback(EStartZenDashboardResult::Completed);
	}
	else
	{
		Callback(EStartZenDashboardResult::LaunchFailed);
	}
}

void FZenDashboardLauncher::CloseZenDashboard()
{
	if (ZenDashboardHandle.IsValid())
	{
		FPlatformProcess::TerminateProc(ZenDashboardHandle);
	}
	else
	{
		UE_LOGF(LogZenDashboardLauncher, Log, "Could not find the Zen Dashboard process handler");
	}
}

void FZenDashboardLauncher::BuildZenDashboard(const FString& Path, const FString& LaunchParameters, StartZenDashboardCallback Callback)
{
	UE_LOGF(LogZenDashboardLauncher, Log, "Could not find the Zen Dashboard executable: %ls. Attempting to build Zen Dashboard.", *Path);

	FString Arguments;
#if PLATFORM_WINDOWS
	FText PlatformName = LOCTEXT("PlatformName_Windows", "Windows");
	Arguments = TEXT("BuildTarget -Target=ZenDashboard -Platform=Win64");
#elif PLATFORM_MAC
	FText PlatformName = LOCTEXT("PlatformName_Mac", "Mac");
	Arguments = TEXT("BuildTarget -Target=ZenDashboard -Platform=Mac");
#elif PLATFORM_LINUX
	FText PlatformName = LOCTEXT("PlatformName_Linux", "Linux");
	Arguments = TEXT("BuildTarget -Target=ZenDashboard -Platform=Linux");
#endif

	IUATHelperModule::Get().CreateUatTask(Arguments, PlatformName, LOCTEXT("BuildingZenDashboard", "Building Zen Dashboard"),
		LOCTEXT("BuildZenDashboardTask", "Build Zen Dashboard Task"), FAppStyle::GetBrush(TEXT("MainFrame.CookContent")), nullptr,
		[this, Path, LaunchParameters, Callback](FString Result, double Time)
		{
			if (Result.Equals(TEXT("Completed")))
			{
#if PLATFORM_MAC
				// On Mac we genereate the path again so that it includes the newly built executable.
				FString NewPath = GetZenDashboardApplicationPath();
				this->StartZenDashboard(NewPath, LaunchParameters, Callback);
#else
				this->StartZenDashboard(Path, LaunchParameters, Callback);
#endif
			}
			else
			{
				Callback(EStartZenDashboardResult::BuildFailed);
			}
		});
}

} // namespace UE::Zen

#undef LOCTEXT_NAMESPACE
