// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/PlatformProcess.h"

#define UE_API

namespace UE::Zen
{

class FZenDashboardLauncher : public TSharedFromThis<FZenDashboardLauncher>
{
public:
	enum class EStartZenDashboardResult : uint32
	{
		Completed = 0,
		BuildFailed = 1,
		LaunchFailed = 2,
	};

	typedef TFunction<void(const EStartZenDashboardResult /*Result*/)> StartZenDashboardCallback;

	UE_API FZenDashboardLauncher();
	UE_API ~FZenDashboardLauncher();
	
	static const TSharedPtr<FZenDashboardLauncher>& Get()
	{
		if (!Instance)
		{
			Instance = MakeShared<FZenDashboardLauncher>();
		}
		return Instance;
	}

	/**
	 * Returns the full path to ZenDashboard.exe for the current engine installation 
	 */
	UE_API FString GetZenDashboardApplicationPath();

	/**
	 * Launches the ZenDashboard executable from the given Path, displays an editor message if it fails.
	 * If the executable is not found, a build process is started.
	 * @param Path The full filename of ZenDashboard.exe to launch
	 * @param Parameters The command line parameters to use when launching the exe
	 */	
	UE_API void StartZenDashboard(const FString& Path, const FString& Parameters = TEXT(""));

	/**
	* Launches the ZenDashboard executable from the given Path, displays an editor message if it fails.
	* If the executable is not found, a build process is started.
	* @param Path The full filename of ZenDashboard.exe to launch
	* @param Parameters The command line parameters to use when launching the exe
	* @param Callback A Callback that will be called when the launch process is completed.
	*/
	UE_API void StartZenDashboard(const FString& Path, const FString& Parameters, StartZenDashboardCallback Callback);

	/**
	* Closes ZenDashboard.exe.
	*/
	UE_API void CloseZenDashboard();

private:
	/*
	 * Attempts building ZenDashboard via UAT, will launch with forwarded parameters if successful.
	 * Assumes that the ZenDashboard Executable in path belongs to this engine.
	 */
	UE_API void BuildZenDashboard(const FString& Path, const FString& LaunchParameters, StartZenDashboardCallback Callback);

private:
	/** The process handler of ZenDashboard. */
	FProcHandle ZenDashboardHandle;

	static UE_API TSharedPtr<FZenDashboardLauncher> Instance;
};

} // namespace UE::Zen

#undef UE_API
