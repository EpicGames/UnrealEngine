// Copyright Epic Games, Inc. All Rights Reserved.

#include "ZenEditor.h"
#include "DerivedDataInformation.h"
#include "Experimental/ZenServerInterface.h"
#include "Framework/Application/SlateApplication.h"
#include "Modules/ModuleManager.h"
#include "SDerivedDataDialogs.h"
#include "SZenDialogs.h"
#include "SZenStatusBar.h"
#include "Styling/AppStyle.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Docking/SDockTab.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"

#define LOCTEXT_NAMESPACE "ZenEditor"

IMPLEMENT_MODULE(FZenEditor, ZenEditor);

static const FName DerivedDataResourceUsageTabName = FName(TEXT("DerivedDataResourceUsage"));
static const FName DerivedDataCacheStatisticsTabName = FName(TEXT("DerivedDataCacheStatistics"));
static const FName ZenServerStatusTabName = FName(TEXT("ZenServerStatus"));

void FZenEditor::StartupModule()
{
	const FSlateIcon ResourceUsageIcon(FAppStyle::GetAppStyleSetName(), "DerivedData.ResourceUsage");

	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(DerivedDataResourceUsageTabName, FOnSpawnTab::CreateRaw(this, &FZenEditor::CreateResourceUsageTab))
		.SetDisplayName(LOCTEXT("DerivedDataResourceUsageTabTitle", "Resource Usage"))
		.SetTooltipText(LOCTEXT("DerivedDataResourceUsageTabToolTipText", "Derived Data Resource Usage"))
		.SetGroup(WorkspaceMenu::GetMenuStructure().GetToolsCategory())
		.SetIcon(ResourceUsageIcon);

	const FSlateIcon CacheStatisticsIcon(FAppStyle::GetAppStyleSetName(), "DerivedData.Cache.Statistics");

	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(DerivedDataCacheStatisticsTabName, FOnSpawnTab::CreateRaw(this, &FZenEditor::CreateCacheStatisticsTab))
		.SetDisplayName(LOCTEXT("DerivedDataCacheStatisticsTabTitle", "Cache Statistics"))
		.SetTooltipText(LOCTEXT("DerivedDataCacheStatisticsTabToolTipText", "Derived Data Cache Statistics"))
		.SetGroup(WorkspaceMenu::GetMenuStructure().GetToolsCategory())
		.SetIcon(CacheStatisticsIcon);

	const FSlateIcon ZenServerIcon(FAppStyle::GetAppStyleSetName(), "Zen.Server");

	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(ZenServerStatusTabName, FOnSpawnTab::CreateRaw(this, &FZenEditor::CreateZenServerStatusTab))
		.SetDisplayName(LOCTEXT("ZenServerStatusTabTitle", "Zen Server Status"))
		.SetTooltipText(LOCTEXT("ZenServerStatusTabToolTipText", "Zen Server Status"))
		.SetGroup(WorkspaceMenu::GetMenuStructure().GetToolsCategory())
		.SetIcon(ZenServerIcon);

#if WITH_RELOAD
	// This code attempts to relaunch the tabs when you reload this module
	if (IsReloadActive() && FSlateApplication::IsInitialized())
	{
		ShowCacheStatisticsTab();
		ShowResourceUsageTab();
		ShowZenServerStatusTab();
	}
#endif // WITH_RELOAD

	FZenStausBarCommands::Register();
}

void FZenEditor::ShutdownModule()
{
	if (FSlateApplication::IsInitialized())
	{
		FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(DerivedDataResourceUsageTabName);

		if (ResourceUsageTab.IsValid())
		{
			ResourceUsageTab.Pin()->RequestCloseTab();
		}

		FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(DerivedDataCacheStatisticsTabName);

		if (CacheStatisticsTab.IsValid())
		{
			CacheStatisticsTab.Pin()->RequestCloseTab();
		}

		FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(ZenServerStatusTabName);

		if (ZenServerStatusTab.IsValid())
		{
			ZenServerStatusTab.Pin()->RequestCloseTab();
		}
	}

	FZenStausBarCommands::Unregister();
}

bool FZenEditor::IsZenEnabled() const
{
	return UE::Zen::IsDefaultServicePresent();
}

TSharedRef<SWidget> FZenEditor::CreateStatusBarWidget()
{
	return SNew(SZenStatusBarWidget);
}

TSharedPtr<SWidget> FZenEditor::CreateResourceUsageDialog()
{
	return SNew(SDerivedDataResourceUsageDialog);
}

TSharedRef<SDockTab> FZenEditor::CreateResourceUsageTab(const FSpawnTabArgs& Args)
{
	return SAssignNew(ResourceUsageTab, SDockTab)
	.TabRole(ETabRole::NomadTab)
	[
		CreateResourceUsageDialog().ToSharedRef()
	];
}

void FZenEditor::ShowResourceUsageTab()
{
	FGlobalTabmanager::Get()->TryInvokeTab(FTabId(DerivedDataResourceUsageTabName));
}

TSharedPtr<SWidget> FZenEditor::CreateCacheStatisticsDialog()
{
	return SNew(SDerivedDataCacheStatisticsDialog);
}

TSharedRef<SDockTab> FZenEditor::CreateCacheStatisticsTab(const FSpawnTabArgs& Args)
{
	return SAssignNew(CacheStatisticsTab, SDockTab)
		.TabRole(ETabRole::NomadTab)
		[
			CreateCacheStatisticsDialog().ToSharedRef()
		];
}

void FZenEditor::ShowCacheStatisticsTab()
{
	FGlobalTabmanager::Get()->TryInvokeTab(FTabId(DerivedDataCacheStatisticsTabName));
}

TSharedPtr<SWidget> FZenEditor::CreateZenStoreDialog()
{
	return SNew(SZenStoreStausDialog);
}

void FZenEditor::ShowZenServerStatusTab()
{
	FGlobalTabmanager::Get()->TryInvokeTab(FTabId(ZenServerStatusTabName));
}

TSharedRef<SDockTab> FZenEditor::CreateZenServerStatusTab(const FSpawnTabArgs& Args)
{
	return SAssignNew(ZenServerStatusTab, SDockTab)
		.TabRole(ETabRole::NomadTab)
		[
			CreateZenStoreDialog().ToSharedRef()
		];
}

void FZenEditor::StartZenServer()
{
	UE::Zen::FZenLocalServiceRunContext RunContext;
	if (UE::Zen::TryGetLocalServiceRunContext(RunContext))
	{
		UE::Zen::StartLocalService(RunContext);
	}
}

void FZenEditor::StopZenServer()
{
	UE::Zen::FZenLocalServiceRunContext RunContext;
	if (UE::Zen::TryGetLocalServiceRunContext(RunContext))
	{
		UE::Zen::StopLocalService(*RunContext.GetDataPath());
	}
}

void FZenEditor::RestartZenServer()
{
	StopZenServer();
	StartZenServer();
}

#undef LOCTEXT_NAMESPACE
