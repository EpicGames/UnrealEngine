// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"

#define UE_API ZENEDITOR_API

class FSpawnTabArgs;
class SDerivedDataCacheSettingsDialog;
class IDerivedDataCacheNotifications;
class SDockTab;
class SWidget;
class SWindow;

/**
 * The module holding all of the UI related pieces for DerivedData
 */
class FZenEditor : public IModuleInterface
{
public:

	/**
	 * Called right after the module DLL has been loaded and the module object has been created
	 */
	UE_API virtual void StartupModule() override;

	/**
	 * Called before the module is unloaded, right before the module object is destroyed.
	 */
	UE_API virtual void ShutdownModule() override;

	UE_API bool IsZenEnabled() const;
	UE_API TSharedRef<SWidget>	CreateStatusBarWidget();

	UE_API void ShowResourceUsageTab();
	UE_API void ShowCacheStatisticsTab();
	UE_API void ShowZenServerStatusTab();
	UE_API void StartZenServer();
	UE_API void StopZenServer();
	UE_API void RestartZenServer();

private:

	UE_API TSharedPtr<SWidget> CreateResourceUsageDialog();
	UE_API TSharedPtr<SWidget> CreateCacheStatisticsDialog();
	UE_API TSharedPtr<SWidget> CreateZenStoreDialog();

	UE_API TSharedRef<SDockTab> CreateResourceUsageTab(const FSpawnTabArgs& Args);
	UE_API TSharedRef<SDockTab> CreateCacheStatisticsTab(const FSpawnTabArgs& Args);
	UE_API TSharedRef<SDockTab> CreateZenServerStatusTab(const FSpawnTabArgs& Args);

	TWeakPtr<SDockTab> ResourceUsageTab;
	TWeakPtr<SDockTab> CacheStatisticsTab;
	TWeakPtr<SDockTab> ZenServerStatusTab;

	TSharedPtr<SWindow>	SettingsWindow;
	TSharedPtr<SDerivedDataCacheSettingsDialog> SettingsDialog;
};


#undef UE_API
