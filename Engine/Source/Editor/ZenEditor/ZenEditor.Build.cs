// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class ZenEditor : ModuleRules
{
	public ZenEditor(ReadOnlyTargetRules Target)
		 : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"DerivedDataCache",
				"DerivedDataWidgets",
				"EditorFramework",
				"EditorSubsystem",
				"Engine",
				"InputCore",
				"MessageLog",
				"OutputLog",
				"Slate",
				"SlateCore",
				"StorageServerWidgets",
				"ToolMenus",
				"ToolWidgets",
				"UATHelper",
				"UnrealEd",
				"WorkspaceMenuStructure",
				"Zen"
			});

		DynamicallyLoadedModuleNames.AddRange(
			new string[] {
				"MainFrame",
			});


		PrivateIncludePathModuleNames.AddRange(
			new string[] {
			});
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
	}
}
